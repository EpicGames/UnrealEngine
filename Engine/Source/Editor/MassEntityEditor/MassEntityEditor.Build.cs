// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class MassEntityEditor : ModuleRules
	{
		public MassEntityEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"MassEntity",
					"SlateCore",
					"UnrealEd",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AIGraph",
					"AssetTools",
					"ComponentVisualizers",
					"DetailCustomizations",
					"EditorSubsystem",
					"GraphEditor",
					"InputCore",
					"KismetWidgets",
					"MassCore",
					"MassDeveloper",
					"Projects",
					"PropertyEditor",
					"RenderCore",
					"RewindDebuggerInterface",
					"RewindDebuggerRuntimeInterface",
					"Slate",
					"ToolMenus",
					"TraceLog",
					"TraceServices",
				}
			);

			if (Target.bBuildDeveloperTools == true)
			{
				PrivateDependencyModuleNames.Add("MessageLog");
				DynamicallyLoadedModuleNames.Add("MassEntityDebugger");
			}
		}
	}
}
