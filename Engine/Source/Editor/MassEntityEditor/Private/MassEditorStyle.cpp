// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEditorStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Interfaces/IPluginManager.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FMeshEditorStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define RootToContentDir StyleSet->RootToContentDir

TSharedPtr<FSlateStyleSet> FMassEntityEditorStyle::StyleSet = nullptr;

FString FMassEntityEditorStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("MassEntityEditorModule"))->GetContentDir() / TEXT("Slate");
	return (ContentDir / RelativePath) + Extension;
}

void FMassEntityEditorStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	const FVector2f Icon8x8(8.0f, 8.0f);
	const FVector2f Icon16x16(16.0f, 16.0f);

	StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());

	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FScrollBarStyle ScrollBar = FAppStyle::GetWidgetStyle<FScrollBarStyle>("ScrollBar");
	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	// State
	{
		FTextBlockStyle StateIcon = FTextBlockStyle(NormalText)
			.SetFont(FAppStyle::Get().GetFontStyle("FontAwesome.12"))
			.SetColorAndOpacity(FLinearColor(230.0f / 255.0f, 230.0f / 255.0f, 230.0f / 255.0f, 0.5f));
		StyleSet->Set("Mass.Icon", StateIcon);

		FTextBlockStyle StateTitle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 12))
			.SetColorAndOpacity(FLinearColor(230.0f / 255.0f, 230.0f / 255.0f, 230.0f / 255.0f));
		StyleSet->Set("Mass.State.Title", StateTitle);

		FEditableTextBoxStyle StateTitleEditableText = FEditableTextBoxStyle()
			.SetTextStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 10))
			.SetBackgroundImageNormal(BOX_BRUSH("Common/TextBox", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageHovered(BOX_BRUSH("Common/TextBox_Hovered", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageFocused(BOX_BRUSH("Common/TextBox_Hovered", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageReadOnly(BOX_BRUSH("Common/TextBox_ReadOnly", FMargin(4.0f / 16.0f)))
			.SetScrollBarStyle(ScrollBar);
		StyleSet->Set("Mass.State.TitleEditableText", StateTitleEditableText);

		StyleSet->Set("Mass.State.TitleInlineEditableText", FInlineEditableTextBlockStyle()
			.SetTextStyle(StateTitle)
			.SetEditableTextBoxStyle(StateTitleEditableText));
	}

	// Task
	{
		FTextBlockStyle TaskTitle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 11))
			.SetColorAndOpacity(FLinearColor(230.0f / 255.0f, 230.0f / 255.0f, 230.0f / 255.0f));
		StyleSet->Set("Mass.Task.Title", TaskTitle);

		FEditableTextBoxStyle TaskTitleEditableText = FEditableTextBoxStyle()
			.SetTextStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 9))
			.SetBackgroundImageNormal(BOX_BRUSH("Common/TextBox", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageHovered(BOX_BRUSH("Common/TextBox_Hovered", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageFocused(BOX_BRUSH("Common/TextBox_Hovered", FMargin(4.0f / 16.0f)))
			.SetBackgroundImageReadOnly(BOX_BRUSH("Common/TextBox_ReadOnly", FMargin(4.0f / 16.0f)))
			.SetScrollBarStyle(ScrollBar);
		StyleSet->Set("Mass.Task.TitleEditableText", TaskTitleEditableText);

		StyleSet->Set("Mass.Task.TitleInlineEditableText", FInlineEditableTextBlockStyle()
			.SetTextStyle(TaskTitle)
			.SetEditableTextBoxStyle(TaskTitleEditableText));
	}

	// Details
	{
		FTextBlockStyle StateTitle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(FLinearColor(230.0f / 255.0f, 230.0f / 255.0f, 230.0f / 255.0f));
		StyleSet->Set("Mass.Details", StateTitle);
	}

	const FLinearColor SelectionColor = FColor(0, 0, 0, 32);
	const FTableRowStyle& NormalTableRowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
	StyleSet->Set("Mass.Selection",
		FTableRowStyle(NormalTableRowStyle)
		.SetActiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetActiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetInactiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetInactiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
		.SetSelectorFocusedBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
	);

	// Class icons
	StyleSet->Set("ClassIcon.MassEntityHandle", new IMAGE_BRUSH_SVG("Icons/AssetIcons/MassEntityHandle_16", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

#undef IMAGE_PLUGIN_BRUSH
#undef RootToContentDir

void FMassEntityEditorStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

FName FMassEntityEditorStyle::GetStyleSetName()
{
	static FName StyleName("MassEntityEditorStyle");
	return StyleName;
}
