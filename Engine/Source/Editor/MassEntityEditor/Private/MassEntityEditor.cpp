// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityEditor.h"
#include "Logging/MessageLog.h"
#include "Framework/Docking/TabManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityEditor)

#define LOCTEXT_NAMESPACE "Mass"

void FMassEditorNotification::Show()
{
	FMessageLog MessageLog(UE::Mass::Editor::MessageLogPageName);

	MessageLog.AddMessage(FTokenizedMessage::Create(Severity))
		->AddToken(FTextToken::Create(Message));

	if (bIncludeSeeOutputLogForDetails)
	{
		MessageLog.AddMessage(FTokenizedMessage::Create(EMessageSeverity::Info))
			->AddToken(FActionToken::Create(LOCTEXT("MassSeeLogForDetails", "See the log for more details.")
				, LOCTEXT("MassSeeLogForDetailsTooltip", "Open the Output Log tab.")
				, FOnActionTokenExecuted::CreateLambda([]()
					{
						FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog"));
					}))
			);
	}

	// Forcing so that even the "Info"-level notifications get shown
	MessageLog.Notify(Message, EMessageSeverity::Info, /*bForce=*/true);
}

#undef LOCTEXT_NAMESPACE
