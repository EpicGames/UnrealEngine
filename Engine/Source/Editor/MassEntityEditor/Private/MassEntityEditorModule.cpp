// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityEditorModule.h"

#include "Features/IModularFeatures.h"
#include "IRewindDebuggerTrackCreator.h"
#include "MassEditorStyle.h"
#include "Modules/ModuleManager.h"
#include "Trace/MassEntityTrack.h"
#include "Trace/MassRewindDebuggerIntegration.h"
#if WITH_UNREAL_DEVELOPER_TOOLS
#include "Engine/World.h"
#include "Logging/MessageLog.h"
#include "MassDebugger.h"
#include "MassEntityEditor.h"
#include "MessageLogModule.h"
#endif // WITH_UNREAL_DEVELOPER_TOOLS

#define LOCTEXT_NAMESPACE "Mass"

IMPLEMENT_MODULE(FMassEntityEditorModule, MassEntityEditor)

void FMassEntityEditorModule::StartupModule()
{
	using namespace UE::Mass;

	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	FMassEntityEditorStyle::Initialize();

#if WITH_UNREAL_DEVELOPER_TOOLS
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowPages = true;
	InitOptions.bShowFilters = true;
	MessageLogModule.RegisterLogListing(Editor::MessageLogPageName
		, FText::FromName(Editor::MessageLogPageName), InitOptions);

	OnWorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddStatic(&FMassEntityEditorModule::OnWorldCleanup);

	FModuleManager::Get().LoadModule("MassEntityDebugger");
#endif // WITH_UNREAL_DEVELOPER_TOOLS

	// RewindDebugger integration
	RuntimeExtension = MakeUnique<FMassRewindDebuggerRuntimeExtension>();
	EntityTrackCreator = MakeUnique<Trace::FEntityTrackCreator>();

	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	ModularFeatures.RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, RuntimeExtension.Get());
	ModularFeatures.RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, EntityTrackCreator.Get());
}

void FMassEntityEditorModule::ShutdownModule()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	// Unregister RewindDebugger integration
	if (RuntimeExtension.IsValid())
	{
		ModularFeatures.UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, RuntimeExtension.Get());
	}
	if (EntityTrackCreator.IsValid())
	{
		ModularFeatures.UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, EntityTrackCreator.Get());
	}
	RuntimeExtension.Reset();
	EntityTrackCreator.Reset();

	ProcessorClassCache.Reset();
	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();

	FMassEntityEditorStyle::Shutdown();

#if WITH_UNREAL_DEVELOPER_TOOLS
	FWorldDelegates::OnWorldCleanup.Remove(OnWorldCleanupHandle);
#endif // WITH_UNREAL_DEVELOPER_TOOLS
}

#if WITH_UNREAL_DEVELOPER_TOOLS
void FMassEntityEditorModule::OnWorldCleanup(UWorld* /*World*/, bool /*bSessionEnded*/, bool /*bCleanupResources*/)
{
	// clearing out messages from the world being cleaned up
	FMessageLog(UE::Mass::Editor::MessageLogPageName).NewPage(FText::FromName(UE::Mass::Editor::MessageLogPageName));
}
#endif // WITH_UNREAL_DEVELOPER_TOOLS

#undef LOCTEXT_NAMESPACE
