// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityEditorSubsystem.h"
#include "MassEntityManager.h"
#include "MassEntitySettings.h"
#include "TickableEditorObject.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassEntityEditorSubsystem)

namespace UE::Mass
{
struct FMassEditorPhaseTickTask
{
	FMassEditorPhaseTickTask(const TSharedRef<FMassProcessingPhaseManager>& InPhaseManager, const EMassProcessingPhase InPhase, const float InDeltaTime)
		: PhaseManager(InPhaseManager)
		, Phase(InPhase)
		, DeltaTime(InDeltaTime)
	{
	}

	static TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMassEditorPhaseTickTask, STATGROUP_TaskGraphTasks);
	}

	static ENamedThreads::Type GetDesiredThread() { return ENamedThreads::GameThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FMassEditorPhaseTickTask);

		PhaseManager->TriggerPhase(Phase, DeltaTime, MyCompletionGraphEvent);
	}
private:
	const TSharedRef<FMassProcessingPhaseManager> PhaseManager;
	const EMassProcessingPhase Phase = EMassProcessingPhase::MAX;
	const float DeltaTime = 0.f;
};

struct FMassEditorTickFunction : FTickableEditorObject
{
	FMassEditorTickFunction(UMassEntityEditorSubsystem& InOwner)
		: Owner(InOwner)
	{}

	//** FTickableEditorObject Interface
	virtual void Tick(float DeltaTime) override
	{
		Owner.Tick(DeltaTime);
	}

	virtual ETickableTickType GetTickableTickType() const override 
	{ 
		return ETickableTickType::Always; 
	}

	virtual TStatId GetStatId() const override
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FMassEditorTickFunction, STATGROUP_Tickables);
	}
	//~ END FTickableEditorObject Interface

	UMassEntityEditorSubsystem& Owner;
};

} // namespace UE::Mass

//-----------------------------------------------------------------------------
// UMassEntityEditorSubsystem
//-----------------------------------------------------------------------------
UMassEntityEditorSubsystem::UMassEntityEditorSubsystem()
	: EntityManager(MakeShareable(new FMassEntityManager(this)))
	// we explicitly configure FMassProcessingPhaseManager to be in Editor mode since in this context we will not
	// get a valid UWorld to deduce that - this subsystem runs for the editor itself, not for worlds loaded in the editor. 
	, PhaseManager(new FMassProcessingPhaseManager(EProcessorExecutionFlags::Editor))
{

}

UMassEntityEditorSubsystem::~UMassEntityEditorSubsystem()
{
	if (EditorTickFunction)
	{
		UE_LOGF(LogMass, Warning, "UMassEntityEditorSubsystem::EditorTickFunction still valid in the destructor indicating that UMassEntityEditorSubsystem::Deinitialize had not been called");
		StopAndCleanUp();
	}
}

UMassEntityEditorSubsystem* UMassEntityEditorSubsystem::Get()
{
	return GEngine ? GEngine->GetEngineSubsystem<UMassEntityEditorSubsystem>() : nullptr;
}

void UMassEntityEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	UMassEntitySettings* Settings = GetMutableDefault<UMassEntitySettings>();
	check(Settings);
	if (Settings->IsInitialized2())
	{
		InitializeMassInternals();
	}
	else
	{
		OnSettingsInitializedHandle = Settings->GetOnInitialized().AddUObject(this, &UMassEntityEditorSubsystem::InitializeMassInternals);
	}

	Super::Initialize(Collection);
}

void UMassEntityEditorSubsystem::Deinitialize()
{
	StopAndCleanUp();
	Super::Deinitialize();
}

bool UMassEntityEditorSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
#if WITH_EDITOR
	return true;
#else
	return false;
#endif
}

void UMassEntityEditorSubsystem::InitializeMassInternals()
{
	// Note that since this is a raw pointer we need to remember to delete it once we're done. It's now done in both 
	// Deinitialize and the destructor (in case Deinitialize call is missed in an unexpected circumstances).
	// Originally we wanted to use a TUniquePtr, but it would require to define the UE::Mass::FMassEditorTickFunction type 
	// in the header file.
	EditorTickFunction = new UE::Mass::FMassEditorTickFunction(*this);

	FMassEntityManagerStorageInitParams InitParams;
	InitParams.Emplace<FMassEntityManager_InitParams_Concurrent>(
		FMassEntityManager_InitParams_Concurrent
		{
			.MaxEntityCount = 1llu << 28,
			.MaxEntitiesPerPage = 1 << 16
		});
	EntityManager->Initialize(InitParams);

	// set up ProcessingPhasesConfig
	UMassEntitySettings* Settings = GetMutableDefault<UMassEntitySettings>();
	check(Settings);
	TConstArrayView<FMassProcessingPhaseConfig> MainPhasesConfig = Settings->GetProcessingPhasesConfig();
	for (int PhaseIndex = 0; PhaseIndex < (int)EMassProcessingPhase::MAX; ++PhaseIndex)
	{
		ProcessingPhasesConfig[PhaseIndex] = MainPhasesConfig[PhaseIndex];
	}

	PhaseManager->Initialize(*this, ProcessingPhasesConfig);

	PhaseManager->Start(EntityManager);
}

void UMassEntityEditorSubsystem::StopAndCleanUp()
{
	if (EditorTickFunction)
	{
		delete EditorTickFunction;
		EditorTickFunction = nullptr;
	}

	PhaseManager->Stop();

	if (OnSettingsInitializedHandle.IsValid())
	{
		UMassEntitySettings* Settings = GetMutableDefault<UMassEntitySettings>();
		check(Settings);
		Settings->GetOnInitialized().Remove(OnSettingsInitializedHandle);
	}
}

void UMassEntityEditorSubsystem::Tick(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMassEntityEditorSubsystem::Tick)
	IsProcessing = true;

	OnPreTickDelegate.Broadcast(DeltaTime);

	FGraphEventRef CompletionEvent;
	for (int PhaseIndex = 0; PhaseIndex < (int)EMassProcessingPhase::MAX; ++PhaseIndex)
	{
		const FGraphEventArray Prerequisites = { CompletionEvent };
		CompletionEvent = TGraphTask<UE::Mass::FMassEditorPhaseTickTask>::CreateTask(&Prerequisites)
			.ConstructAndDispatchWhenReady(PhaseManager, EMassProcessingPhase(PhaseIndex), DeltaTime);
	}

	if (CompletionEvent.IsValid())
	{
		CompletionEvent->Wait();
	}

	OnPostTickDelegate.Broadcast(DeltaTime);

	IsProcessing = false;
}

void UMassEntityEditorSubsystem::RegisterDynamicProcessor(UMassProcessor& Processor)
{
	checkf(!IsProcessing, TEXT("Unable to add dynamic processors to Mass during processing."));
	PhaseManager->RegisterDynamicProcessor(Processor);
}

void UMassEntityEditorSubsystem::UnregisterDynamicProcessor(UMassProcessor& Processor)
{
	checkf(!IsProcessing, TEXT("Unable to remove dynamic processors to Mass during processing."));
	PhaseManager->UnregisterDynamicProcessor(Processor);
}
