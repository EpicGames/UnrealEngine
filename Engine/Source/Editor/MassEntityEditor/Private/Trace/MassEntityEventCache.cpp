// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityEventCache.h"

#include "Common/ProviderLock.h"
#include "Misc/ScopeRWLock.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Frames.h"

namespace UE::Mass::Trace
{

void FEntityEventCache::Update(
	const IMassTraceProvider& Provider,
	const TraceServices::IAnalysisSession& Session)
{
	{
		TraceServices::FProviderReadScopeLock ReadLock(Provider);
		const uint64 CurrentCounter = Provider.GetUpdateCounter();
		if (CurrentCounter == LastUpdateCounter)
		{
			return;
		}
		LastUpdateCounter = CurrentCounter;

		FWriteScopeLock WriteLock(CacheLock);
		EntityEvents.Reset();
		ArchetypeEntities.Reset();
		ActiveArchetypeIds.Reset();

		const uint64 TotalEvents = Provider.GetEntityEventCount();
		if (TotalEvents == 0)
		{
			return;
		}

		Provider.EnumerateEntityEvents(0, TotalEvents,
			[this](const FEntityEventRecord& Record, uint64 /*EventIndex*/)
			{
				FCachedEntityEventRecord& CachedRecord = EntityEvents.FindOrAdd(Record.Entity).AddDefaulted_GetRef();
				static_cast<FEntityEventRecord&>(CachedRecord) = Record;
				ArchetypeEntities.FindOrAdd(Record.ArchetypeID).Add(Record.Entity);
				ActiveArchetypeIds.Add(Record.ArchetypeID);
			});
	}

	ConvertTimesToRecordingTime(Session);

	// Sort each entity's events by ProfileTime so consumers can rely on ascending
	// chronological order.  We use ProfileTime rather than Time (ElapsedTime) because
	// ElapsedTime freezes during pauses — events in different frames can share the
	// same ElapsedTime value, and sorting by it could interleave frames and break
	// the FrameIndex-based batching in ResolveArchetypesAtScrubTime.
	{
		FWriteScopeLock WriteLock(CacheLock);
		for (TPair<uint64, TArray<FCachedEntityEventRecord>>& Pair : EntityEvents)
		{
			Pair.Value.Sort([](const FCachedEntityEventRecord& A, const FCachedEntityEventRecord& B)
			{
				return A.ProfileTime < B.ProfileTime;
			});
		}
	}
}

void FEntityEventCache::ConvertTimesToRecordingTime(const TraceServices::IAnalysisSession& Session)
{
	TraceServices::FAnalysisSessionReadScope SessionReadScope(Session);
	const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(Session);

	// Resolve frame indices from ProfileTime.
	for (TPair<uint64, TArray<FCachedEntityEventRecord>>& Pair : EntityEvents)
	{
		for (FCachedEntityEventRecord& Event : Pair.Value)
		{
			Event.FrameIndex = FrameProvider.GetFrameNumberForTimestamp(TraceFrameType_Game, Event.ProfileTime);
		}
	}

	// For new traces, Event.Time is already in ElapsedTime (recording time) space —
	// set at trace emission via FMassTrace::SetRecordingTime / FObjectTrace::GetWorldElapsedTime.
	//
	// For old traces that lack the RecordingTime field, the analyzer falls back to
	// ProfileTime for Event.Time (Time == ProfileTime). These are raw absolute values
	// that are incomparable to the RewindDebugger's ElapsedTime-based scrub time.
	// Normalize them relative to the first frame's start time so they become
	// approximately comparable (not pause-aligned, but functional).
	const uint64 FrameCount = FrameProvider.GetFrameCount(TraceFrameType_Game);
	if (FrameCount == 0)
	{
		return;
	}

	const TraceServices::FFrame* FirstFrame = FrameProvider.GetFrame(TraceFrameType_Game, 0);
	if (!FirstFrame)
	{
		return;
	}

	const double ProfileStart = FirstFrame->StartTime;
	for (TPair<uint64, TArray<FCachedEntityEventRecord>>& Pair : EntityEvents)
	{
		for (FCachedEntityEventRecord& Event : Pair.Value)
		{
			// Events from old traces have Time == ProfileTime (analyzer fallback).
			// Normalize these to be relative to trace start. Events with proper
			// RecordingTime (Time != ProfileTime) are left untouched.
			if (Event.Time == Event.ProfileTime)
			{
				Event.Time -= ProfileStart;
			}
		}
	}
}

TConstArrayView<FEntityEventCache::FCachedEntityEventRecord> FEntityEventCache::GetEntityEvents(const uint64 EntityId) const
{
	FReadScopeLock ReadLock(CacheLock);
	if (const TArray<FCachedEntityEventRecord>* Events = EntityEvents.Find(EntityId))
	{
		return MakeArrayView(*Events);
	}
	return {};
}

const TSet<uint64>* FEntityEventCache::GetEntitiesForArchetype(const uint64 ArchetypeId) const
{
	FReadScopeLock ReadLock(CacheLock);
	return ArchetypeEntities.Find(ArchetypeId);
}

} // namespace UE::Mass::Trace
