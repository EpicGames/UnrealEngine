// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/ScopeRWLock.h"
#include "Trace/MassTraceTypes.h"

namespace TraceServices
{
class IAnalysisSession;
}

namespace UE::Mass::Trace
{
class IMassTraceProvider;
/**
 * Caches entity events indexed by entity and archetype for efficient sub-track querying.
 * The IMassTraceProvider only offers sequential enumeration, so this cache builds
 * per-entity and per-archetype indices to avoid O(N) scans in every sub-track update.
 */
class FEntityEventCache
{
public:
	/** Extended event record with frame index resolved at cache build time. */
	struct FCachedEntityEventRecord : FEntityEventRecord
	{
		/** Game frame index this event belongs to. Events in the same frame share the same index. */
		uint32 FrameIndex = 0;
	};

	/** Rebuild indices from the provider if GetUpdateCounter() has changed. */
	void Update(const IMassTraceProvider& Provider,
		const TraceServices::IAnalysisSession& Session);

	/** Get all events for a specific entity. Returns empty array if entity not found. */
	TConstArrayView<FCachedEntityEventRecord> GetEntityEvents(uint64 EntityId) const;

	/** Get all entity IDs that have events referencing a specific archetype. */
	const TSet<uint64>* GetEntitiesForArchetype(uint64 ArchetypeId) const;

	/** Get all archetype IDs that appear in the event data. */
	const TSet<uint64>& GetActiveArchetypeIds() const
	{
		FReadScopeLock ReadLock(CacheLock);
		return ActiveArchetypeIds;
	}

	/** Whether any data has been cached. */
	bool HasData() const
	{
		FReadScopeLock ReadLock(CacheLock);
		return EntityEvents.Num() > 0;
	}

private:
	/** Resolve frame indices from ProfileTime and ensure Event.Time is in ElapsedTime
	 * (recording time) space. New traces already carry correct ElapsedTime from emission;
	 * old traces lacking the RecordingTime field have Time == ProfileTime and are normalized
	 * relative to the first frame's start time as a backward-compatibility fallback. */
	void ConvertTimesToRecordingTime(const TraceServices::IAnalysisSession& Session);

	/** Entity ID -> array of events, sorted by ascending ProfileTime after Update() */
	TMap<uint64, TArray<FCachedEntityEventRecord>> EntityEvents;

	/** Archetype ID -> set of entity IDs that reference it */
	TMap<uint64, TSet<uint64>> ArchetypeEntities;

	/** All archetype IDs that appear in the event data */
	TSet<uint64> ActiveArchetypeIds;

	/** Last seen update counter from the provider */
	uint64 LastUpdateCounter = 0;

	/** Protects all cache state from concurrent read/write access */
	mutable FRWLock CacheLock;
};
} // namespace UE::Mass::Trace
