// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassEntityTrack.h"
#include "Common/ProviderLock.h"
#include "IRewindDebugger.h"
#include "Mass/EntityHandle.h"
#include "MassEntityEventCache.h"
#include "SMassEntityDetailsView.h"
#include "Trace/MassTraceTypes.h"
#include "TraceServices/Model/AnalysisSession.h"

#define LOCTEXT_NAMESPACE "MassEntityTrack"

namespace UE::Mass::Trace
{
static constexpr FLinearColor CreatedColor(0.2f, 0.9f, 0.2f, 1.0f);
static constexpr FLinearColor ArchetypeChangeColor(0.9f, 0.9f, 0.2f, 1.0f);
static constexpr FLinearColor DestroyedColor(0.9f, 0.2f, 0.2f, 1.0f);

static FLinearColor ColorFromArchetypeId(const uint64 ArchetypeId)
{
	// Thorough bit mixing to produce distinct hues from pointer-based IDs
	// that tend to be numerically close to each other.
	uint64 H = ArchetypeId;
	H ^= H >> 33;
	H *= 0xff51afd7ed558ccdULL;
	H ^= H >> 33;
	H *= 0xc4ceb9fe1a85ec53ULL;
	H ^= H >> 33;

	// Use golden ratio to spread hues evenly
	constexpr float GoldenRatio = 0.618033988749895f;
	const float Hue = FMath::Frac(static_cast<float>(H & 0xFFFF) / 65535.0f + GoldenRatio);

	// Vary saturation and value slightly for additional differentiation
	const uint8 Saturation = 180 + static_cast<uint8>((H >> 16) % 40); // 180-219
	const uint8 Value = 190 + static_cast<uint8>((H >> 24) % 50); // 190-239

	return FLinearColor::MakeFromHSV8(
		static_cast<uint8>(Hue * 255.0f),
		Saturation,
		Value
	).CopyWithNewOpacity(0.55f);
}

static FText BuildArchetypeTooltip(
	const IMassTraceProvider& Provider,
	const uint64 ArchetypeID)
{
	const FArchetypeInfo* Info = nullptr;
	{
		TraceServices::FProviderReadScopeLock ReadLock(Provider);
		Info = Provider.FindArchetypeById(ArchetypeID);
	}
	if (!Info || Info->Fragments.Num() == 0)
	{
		return FText();
	}
	FString Name;
	constexpr int32 MaxDisplayedFragments = 3;
	for (int32 Idx = 0; Idx < FMath::Min(MaxDisplayedFragments, Info->Fragments.Num()); ++Idx)
	{
		if (Idx > 0)
		{
			Name += TEXT(", ");
		}
		Name += Info->Fragments[Idx]->Name;
	}
	if (Info->Fragments.Num() > MaxDisplayedFragments)
	{
		Name += FString::Printf(TEXT(" (+%d)"), Info->Fragments.Num() - MaxDisplayedFragments);
	}
	return FText::FromString(MoveTemp(Name));
}


//----------------------------------------------------------------------//
// FEntityTrackCreator
//----------------------------------------------------------------------//

FName FEntityTrackCreator::GetTargetTypeNameInternal() const
{
	static const FName Name(TEXT("MassEntityHandle"));
	return Name;
}

FName FEntityTrackCreator::GetNameInternal() const
{
	static const FName Name(TEXT("MassEntity"));
	return Name;
}

void FEntityTrackCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	Types.Add({ FName(TEXT("MassEntity")), LOCTEXT("MassEntity", "Mass Entity") });
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FEntityTrackCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	const TraceServices::IAnalysisSession* Session = IRewindDebugger::Instance()->GetAnalysisSession();
	if (Session)
	{
		const IMassTraceProvider& Provider = ReadMassTraceProvider(*Session);
		Cache->Update(Provider, *Session);
	}

	return MakeShared<FEntityTrack>(InObjectId.GetMainId(), Cache);
}

bool FEntityTrackCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	const TraceServices::IAnalysisSession* Session = IRewindDebugger::Instance()->GetAnalysisSession();
	if (!Session)
	{
		return false;
	}

	const IMassTraceProvider& Provider = ReadMassTraceProvider(*Session);
	Cache->Update(Provider, *Session);
	return Cache->GetEntityEvents(InObjectId.GetMainId()).Num() > 0;
}

//----------------------------------------------------------------------//
// FEntityTrack
//----------------------------------------------------------------------//
FEntityTrack::FEntityTrack(const uint64 InEntityId, TSharedRef<FEntityEventCache> InCache)
	: EntityId(InEntityId)
	, Cache(MoveTemp(InCache))
{
	EventData = MakeShared<SEventTimelineView::FTimelineEventData>();
	Icon = FSlateIcon("EditorStyle", "ClassIcon.Object");

	const FMassEntityHandle EntityHandle(FMassEntityHandle::FromNumber(InEntityId));
	DisplayName = LOCTEXT("EntityTrack", "Composition");
}

bool FEntityTrack::UpdateInternal()
{
	// Always update the details view scrub time, even when throttling timeline rebuilds
	if (DetailsView.IsValid())
	{
		DetailsView->SetScrubTime(IRewindDebugger::Instance()->GetScrubTime());
	}

	TicksSinceEventRebuild++;
	if (TicksSinceEventRebuild <= 10)
	{
		return false;
	}
	TicksSinceEventRebuild = 0;

	EventData->EventPoints.Reset();
	EventData->EventWindows.Reset();

	const TConstArrayView<FEntityEventCache::FCachedEntityEventRecord> Events = Cache->GetEntityEvents(EntityId);
	if (Events.Num() == 0)
	{
		return false;
	}

	const TraceServices::IAnalysisSession* Session = IRewindDebugger::Instance()->GetAnalysisSession();
	if (Session == nullptr)
	{
		return false;
	}

	const TRange<double> ViewRange = IRewindDebugger::Instance()->GetCurrentViewRange();
	const double StartTime = ViewRange.GetLowerBoundValue();
	const double EndTime = ViewRange.GetUpperBoundValue();

	const IMassTraceProvider& Provider = ReadMassTraceProvider(*Session);

	double WindowStart = -1.0;
	uint64 CurrentArchetypeID = 0;

	// Closes the current archetype window by adding it to EventData->EventWindows.
	// Uses the provider to build a tooltip from the archetype's fragment names.
	// bStillActive marks the window as the entity's current (not yet closed) archetype.
	auto CloseCurrentWindow = [this, &Provider, &WindowStart, &CurrentArchetypeID](const double WindowEnd, const bool bStillActive)
	{
		if (WindowStart < 0.0)
		{
			return;
		}
		const FText Tooltip(BuildArchetypeTooltip(Provider, CurrentArchetypeID));
		FLinearColor WindowColor = ColorFromArchetypeId(CurrentArchetypeID);
		if (bStillActive)
		{
			// Brighter and more opaque for the still-active window
			WindowColor = WindowColor.CopyWithNewOpacity(0.85f);
		}
		EventData->EventWindows.Add({
			WindowStart, WindowEnd,
			Tooltip,
			bStillActive ? LOCTEXT("Active", "(active)") : FText(),
			WindowColor
		});
		WindowStart = -1.0;
		CurrentArchetypeID = 0;
	};

	for (const FEntityEventCache::FCachedEntityEventRecord& Event : Events)
	{
		const bool bInViewRange = Event.Time >= StartTime && Event.Time <= EndTime;

		switch (Event.Operation)
		{
		case EEntityEventType::Created:
			WindowStart = Event.Time;
			CurrentArchetypeID = Event.ArchetypeID;
			if (bInViewRange)
			{
				const FText Tooltip(BuildArchetypeTooltip(Provider, Event.ArchetypeID));
				EventData->EventPoints.Add({Event.Time, LOCTEXT("Created", "Created"), Tooltip, CreatedColor});
			}
			break;

		case EEntityEventType::ArchetypeChange:
			// Close previous archetype window, start new one
			CloseCurrentWindow(Event.Time, /*bStillActive*/ false);
			WindowStart = Event.Time;
			CurrentArchetypeID = Event.ArchetypeID;
			if (bInViewRange)
			{
				const FText Tooltip(BuildArchetypeTooltip(Provider, Event.ArchetypeID));
				EventData->EventPoints.Add({Event.Time, LOCTEXT("ArchetypeChange", "Archetype Change"), Tooltip, ArchetypeChangeColor});
			}
			break;

		case EEntityEventType::Destroyed:
			CloseCurrentWindow(Event.Time, /*bStillActive*/ false);
			if (bInViewRange)
			{
				EventData->EventPoints.Add({Event.Time, LOCTEXT("Destroyed", "Destroyed"), FText(), DestroyedColor});
			}
			break;

		default:
			break;
		}
	}

	// Entity still alive — snap window to recording duration and mark as active
	if (WindowStart >= 0.0)
	{
		const double RecordingEnd = IRewindDebugger::Instance()->GetRecordingDuration();
		CloseCurrentWindow(RecordingEnd, /*bStillActive*/ true);
	}

	return false;
}

TSharedPtr<SWidget> FEntityTrack::GetTimelineViewInternal()
{
	return SNew(SEventTimelineView)
		.ViewRange_Lambda([]()
		{
			return IRewindDebugger::Instance()->GetCurrentViewRange();
		})
		.EventData_Raw(this, &FEntityTrack::GetEventData);
}

TSharedPtr<SWidget> FEntityTrack::GetDetailsViewInternal()
{
	if (!DetailsView.IsValid())
	{
		SAssignNew(DetailsView, SMassEntityDetailsView)
			.EntityId(EntityId)
			.Cache(&Cache.Get());
	}
	return DetailsView;
}

TSharedPtr<SEventTimelineView::FTimelineEventData> FEntityTrack::GetEventData() const
{
	return EventData;
}

FSlateIcon FEntityTrack::GetIconInternal()
{
	return Icon;
}

FName FEntityTrack::GetNameInternal() const
{
	return FName(*FString::Printf(TEXT("MassEntity_%llu"), EntityId));
}

FText FEntityTrack::GetDisplayNameInternal() const
{
	return DisplayName;
}

bool FEntityTrack::HasDebugDataInternal() const
{
	return Cache->GetEntityEvents(EntityId).Num() > 0;
}

FText FEntityTrack::GetStepCommandTooltipInternal(const RewindDebugger::EStepMode StepMode) const
{
	using RewindDebugger::EStepMode;
	switch (StepMode)
	{
	case EStepMode::Forward:
		return LOCTEXT("StepForwardTooltip", "Step to next event in Mass Entity track");
	case EStepMode::Backward:
		return LOCTEXT("StepBackwardTooltip", "Step to previous event in Mass Entity track");
	default:
		ensureMsgf(false, TEXT("Unhandled step mode value: %d"), EnumToUnderlyingType(StepMode));
		break;
	}
	return {};
}

TOptional<double> FEntityTrack::GetStepFrameTimeInternal(const RewindDebugger::EStepMode StepMode, const FScrubTimeInformation& CurrentScrubTime) const
{
	const TConstArrayView<FEntityEventCache::FCachedEntityEventRecord> Events = Cache->GetEntityEvents(EntityId);
	if (Events.Num() == 0)
	{
		return {};
	}

	const double ScrubTime = CurrentScrubTime.ElapsedTime;

	using RewindDebugger::EStepMode;
	if (StepMode == EStepMode::Forward)
	{
		// Find the first event strictly after the current scrub time
		for (const FEntityEventCache::FCachedEntityEventRecord& Event : Events)
		{
			if (Event.Time > ScrubTime + UE_DOUBLE_SMALL_NUMBER)
			{
				return Event.Time;
			}
		}
	}
	else
	{
		// Find the last event strictly before the current scrub time
		for (int32 Index = Events.Num() - 1; Index >= 0; --Index)
		{
			if (Events[Index].Time < ScrubTime - UE_DOUBLE_SMALL_NUMBER)
			{
				return Events[Index].Time;
			}
		}
	}

	// No more events in this direction — preserve current position
	return ScrubTime;
}

} // namespace UE::Mass::Trace

#undef LOCTEXT_NAMESPACE
