// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRewindDebuggerTrackCreator.h"
#include "MassEntityEventCache.h"
#include "RewindDebuggerTrack.h"
#include "SEventTimelineView.h"

class SMassEntityDetailsView;

namespace UE::Mass::Trace
{

/**
 * Per-entity leaf track showing lifecycle events (created, archetype change, destroyed)
 * on a timeline with color-coded points and lifetime windows.
 */
class FEntityTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FEntityTrack(uint64 InEntityId, TSharedRef<FEntityEventCache> InCache);

	uint64 GetEntityId() const
	{
		return EntityId;
	}

private:
	//~ FRewindDebuggerTrack interface
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual FSlateIcon GetIconInternal() override;
	virtual FName GetNameInternal() const override;
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override
	{
		return EntityId;
	}
	virtual bool HasDebugDataInternal() const override;
	virtual FText GetStepCommandTooltipInternal(RewindDebugger::EStepMode StepMode) const override;
	virtual TOptional<double> GetStepFrameTimeInternal(RewindDebugger::EStepMode StepMode, const FScrubTimeInformation& CurrentScrubTime) const override;

	TSharedPtr<SEventTimelineView::FTimelineEventData> GetEventData() const;

	uint64 EntityId = 0;
	TSharedRef<FEntityEventCache> Cache;

	mutable TSharedPtr<SEventTimelineView::FTimelineEventData> EventData;
	mutable int32 TicksSinceEventRebuild = 0;

	TSharedPtr<SMassEntityDetailsView> DetailsView;

	FText DisplayName;
	FSlateIcon Icon;
};

/**
 * Track creator for individual Mass entity tracks.
 * Creates FEntityTrack under FMassEntityHandle object tracks.
 */
class FEntityTrackCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const override;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;

	mutable TSharedRef<FEntityEventCache> Cache = MakeShared<FEntityEventCache>();
};

} // UE::Mass::Trace
