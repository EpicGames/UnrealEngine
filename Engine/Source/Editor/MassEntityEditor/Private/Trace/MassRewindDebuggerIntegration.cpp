// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassRewindDebuggerIntegration.h"
#include "Trace/Trace.h"

void FMassRewindDebuggerRuntimeExtension::RecordingStarted()
{
	UE::Trace::ToggleChannel(TEXT("MassChannel"), true);
}

void FMassRewindDebuggerRuntimeExtension::RecordingStopped()
{
	UE::Trace::ToggleChannel(TEXT("MassChannel"), false);
}
