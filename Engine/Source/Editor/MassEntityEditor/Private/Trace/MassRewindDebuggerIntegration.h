// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"

/**
 * Runtime extension that enables/disables the Mass trace channel
 * when RewindDebugger recording starts/stops.
 */
class FMassRewindDebuggerRuntimeExtension final : public IRewindDebuggerRuntimeExtension
{
protected:
	virtual void RecordingStarted() override;
	virtual void RecordingStopped() override;
};
