// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMassEntityDetailsView.h"
#include "Common/ProviderLock.h"
#include "IRewindDebugger.h"
#include "MassEntityEventCache.h"
#include "Trace/MassTraceTypes.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "MassEntityDetailsView"

namespace UE::MassEntityDetailsView::Private
{
static constexpr FLinearColor AddedColor(0.2f, 0.9f, 0.2f, 1.0f);
static constexpr FLinearColor RemovedColor(0.9f, 0.3f, 0.3f, 1.0f);
static constexpr FLinearColor UnchangedColor(0.8f, 0.8f, 0.8f, 1.0f);
static constexpr FLinearColor ReaddedColor(0.6f, 0.6f, 0.9f, 1.0f);
static constexpr FLinearColor InfoColor(0.5f, 0.5f, 0.5f, 1.0f);

static const FName PreviousColumnName(TEXT("Previous"));
static const FName PreviousStepColumnName(TEXT("PrevStep"));
static const FName CurrentColumnName(TEXT("Current"));
static const FName CurrentStepColumnName(TEXT("CurrStep"));
static const FName TypeColumnName(TEXT("Type"));
static const FName SizeColumnName(TEXT("Size"));

} // namespace UE::MassEntityDetailsView::Private

/**
 * Row widget for the diff table.
 */
class SMassArchetypeDiffRow : public SMultiColumnTableRow<TSharedPtr<SMassEntityDetailsView::FFragmentEntry>>
{
	using FEntry = SMassEntityDetailsView::FFragmentEntry;
	using EDiffStatus = SMassEntityDetailsView::EDiffStatus;

public:
	SLATE_BEGIN_ARGS(SMassArchetypeDiffRow) {}
		SLATE_ARGUMENT(TSharedPtr<FEntry>, Entry)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Entry = InArgs._Entry;
		SMultiColumnTableRow::Construct(SMultiColumnTableRow::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		using namespace UE::MassEntityDetailsView::Private;

		if (!Entry.IsValid() || !Entry->Info)
		{
			return SNullWidget::NullWidget;
		}

		const FText FragName = FText::FromString(Entry->Info->Name);

		if (ColumnName == PreviousColumnName)
		{
			// Removed elements show in the Previous column with - prefix and red
			if (Entry->Status == EDiffStatus::Removed)
			{
				return SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("PrefixRemove", "- {0}"), FragName))
					.ColorAndOpacity(RemovedColor);
			}
			// Show fragment name if it was in the previous archetype
			if (Entry->bHasPrevious)
			{
				return SNew(STextBlock)
					.Text(FragName)
					.ColorAndOpacity(UnchangedColor);
			}
			return SNullWidget::NullWidget;
		}

		if (ColumnName == PreviousStepColumnName)
		{
			if (Entry->RemovedAtStep > 0)
			{
				return SNew(STextBlock)
					.Text(FText::AsNumber(Entry->RemovedAtStep))
					.ColorAndOpacity(RemovedColor);
			}
			return SNullWidget::NullWidget;
		}

		if (ColumnName == CurrentColumnName)
		{
			// Removed elements are shown in Previous column, not here
			if (Entry->Status == EDiffStatus::Removed)
			{
				return SNullWidget::NullWidget;
			}

			FText DisplayText;
			FLinearColor Color = UnchangedColor;

			switch (Entry->Status)
			{
			case EDiffStatus::Unchanged:
				DisplayText = FragName;
				break;
			case EDiffStatus::Added:
				Color = AddedColor;
				DisplayText = FText::Format(LOCTEXT("PrefixAdd", "+ {0}"), FragName);
				break;
			case EDiffStatus::Readded:
				Color = ReaddedColor;
				DisplayText = FText::Format(LOCTEXT("PrefixReadd", "~ {0}"), FragName);
				break;
			default:
				break;
			}

			return SNew(STextBlock)
				.Text(DisplayText)
				.ColorAndOpacity(Color);
		}

		if (ColumnName == CurrentStepColumnName)
		{
			if (Entry->AddedAtStep > 0)
			{
				const FLinearColor Color = Entry->Status == EDiffStatus::Readded ? ReaddedColor : AddedColor;
				return SNew(STextBlock)
					.Text(FText::AsNumber(Entry->AddedAtStep))
					.ColorAndOpacity(Color);
			}
			return SNullWidget::NullWidget;
		}

		if (ColumnName == TypeColumnName)
		{
			return SNew(STextBlock)
				.Text(GetTypeText())
				.ColorAndOpacity(InfoColor);
		}

		if (ColumnName == SizeColumnName)
		{
			return SNew(STextBlock)
				.Text(FText::Format(LOCTEXT("SizeFmt", "{0}b"), FText::AsNumber(Entry->Info->Size)))
				.ColorAndOpacity(InfoColor);
		}

		return SNullWidget::NullWidget;
	}

private:
	FText GetTypeText() const
	{
		using namespace UE::Mass::Trace;

		switch (Entry->Info->Type)
		{
		case EFragmentType::Fragment:
			return LOCTEXT("TFragment", "Fragment");
		case EFragmentType::Tag:
			return LOCTEXT("TTag", "Tag");
		case EFragmentType::Shared:
			return LOCTEXT("TShared", "Shared");
		case EFragmentType::Sparse:
			return LOCTEXT("TSparse", "Sparse");
		case EFragmentType::ConstShared:
			return LOCTEXT("TConstShared", "Const Shared");
		case EFragmentType::SparseTag:
			return LOCTEXT("TSparseTag", "Sparse Tag");
		default:
			return LOCTEXT("TUnknown", "Unknown");
		}
	}

	TSharedPtr<FEntry> Entry;
};

//----------------------------------------------------------------------//
// SMassEntityDetailsView
//----------------------------------------------------------------------//

void SMassEntityDetailsView::Construct(const FArguments& InArgs)
{
	using namespace UE::MassEntityDetailsView::Private;
	EntityId = InArgs._EntityId;
	Cache = InArgs._Cache;

	SortColumn = CurrentColumnName;

	HeaderRow = SNew(SHeaderRow)
		+ SHeaderRow::Column(PreviousColumnName)
		.DefaultLabel(LOCTEXT("ColPrev", "Previous"))
		.FillWidth(1.0f)
		.SortMode(this, &SMassEntityDetailsView::GetSortModeForColumn, PreviousColumnName)
		.OnSort(this, &SMassEntityDetailsView::OnSortModeChanged)

		+ SHeaderRow::Column(PreviousStepColumnName)
		.DefaultLabel(LOCTEXT("ColPrevStep", "Step"))
		.FillWidth(0.2f)
		.SortMode(this, &SMassEntityDetailsView::GetSortModeForColumn, PreviousStepColumnName)
		.OnSort(this, &SMassEntityDetailsView::OnSortModeChanged)

		+ SHeaderRow::Column(CurrentColumnName)
		.DefaultLabel(LOCTEXT("ColCurr", "Current"))
		.FillWidth(1.0f)
		.SortMode(this, &SMassEntityDetailsView::GetSortModeForColumn, CurrentColumnName)
		.OnSort(this, &SMassEntityDetailsView::OnSortModeChanged)

		+ SHeaderRow::Column(CurrentStepColumnName)
		.DefaultLabel(LOCTEXT("ColCurrStep", "Step"))
		.FillWidth(0.2f)
		.SortMode(this, &SMassEntityDetailsView::GetSortModeForColumn, CurrentStepColumnName)
		.OnSort(this, &SMassEntityDetailsView::OnSortModeChanged)

		+ SHeaderRow::Column(TypeColumnName)
		.DefaultLabel(LOCTEXT("ColType", "Type"))
		.FillWidth(0.5f)
		.SortMode(this, &SMassEntityDetailsView::GetSortModeForColumn, TypeColumnName)
		.OnSort(this, &SMassEntityDetailsView::OnSortModeChanged)

		+ SHeaderRow::Column(SizeColumnName)
		.DefaultLabel(LOCTEXT("ColSize", "Size"))
		.FillWidth(0.3f)
		.SortMode(this, &SMassEntityDetailsView::GetSortModeForColumn, SizeColumnName)
		.OnSort(this, &SMassEntityDetailsView::OnSortModeChanged);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 2.0f)
		[
			SAssignNew(MergeInfoText, STextBlock)
			.Font(FCoreStyle::GetDefaultFontStyle("Italic", 8))
			.ColorAndOpacity(FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f)))
			.Visibility(EVisibility::Collapsed)
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(ListView, SListView<TSharedPtr<FFragmentEntry>>)
			.ListItemsSource(&FragmentEntries)
			.OnGenerateRow(this, &SMassEntityDetailsView::OnGenerateRow)
			.SelectionMode(ESelectionMode::None)
			.HeaderRow(HeaderRow)
		]
	];
}

void SMassEntityDetailsView::SetScrubTime(const double Time)
{
	CurrentScrubTime = Time;
}

void SMassEntityDetailsView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const double ScrubTime = IRewindDebugger::Instance()->GetScrubTime();
	SetScrubTime(ScrubTime);

	const FArchetypeResolution Resolution = ResolveArchetypesAtScrubTime();

	if (Resolution.CurrentArchetype != LastDisplayedArchetypeId ||
		Resolution.PreviousArchetype != LastDisplayedPreviousArchetypeId ||
		Resolution.MergeCount != LastDisplayedMergeCount)
	{
		LastDisplayedArchetypeId = Resolution.CurrentArchetype;
		LastDisplayedPreviousArchetypeId = Resolution.PreviousArchetype;
		LastDisplayedArchetypeBeforeBatch = Resolution.ArchetypeBeforeBatch;
		LastDisplayedMergeCount = Resolution.MergeCount;
		LastBatchArchetypeIds = Resolution.BatchArchetypeIds;
		RebuildFragmentList();
	}
}

TSharedRef<ITableRow> SMassEntityDetailsView::OnGenerateRow(
	TSharedPtr<FFragmentEntry> Entry,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMassArchetypeDiffRow, OwnerTable).Entry(Entry);
}

SMassEntityDetailsView::FArchetypeResolution SMassEntityDetailsView::ResolveArchetypesAtScrubTime() const
{
	using namespace UE::Mass::Trace;

	FArchetypeResolution Result;

	if (!Cache)
	{
		return Result;
	}

	TConstArrayView<FEntityEventCache::FCachedEntityEventRecord> Events = Cache->GetEntityEvents(EntityId);
	uint32 LastFrameIndex = MAX_uint32;
	int32 CurrentBatchChangeCount = 0;
	TArray<uint64> CurrentBatchArchetypeIds;

	for (const FEntityEventCache::FCachedEntityEventRecord& Record : Events)
	{
		if (Record.Time > CurrentScrubTime + UE_DOUBLE_SMALL_NUMBER)
		{
			break;
		}

		// Track same-frame batching using FrameIndex (resolved from ProfileTime)
		if (Record.FrameIndex != LastFrameIndex)
		{
			Result.ArchetypeBeforeBatch = Result.CurrentArchetype;
			LastFrameIndex = Record.FrameIndex;
			CurrentBatchChangeCount = 0;
			CurrentBatchArchetypeIds.Reset();
		}

		// Previous archetype is always the archetype from the prior event
		switch (Record.Operation)
		{
		case EEntityEventType::Created:
			Result.ArchetypeBeforeBatch = 0;
			Result.PreviousArchetype = 0;
			Result.CurrentArchetype = Record.ArchetypeID;
			CurrentBatchArchetypeIds.Add(Record.ArchetypeID);
			CurrentBatchChangeCount++;
			break;
		case EEntityEventType::ArchetypeChange:
			Result.PreviousArchetype = Result.CurrentArchetype;
			Result.CurrentArchetype = Record.ArchetypeID;
			CurrentBatchArchetypeIds.Add(Record.ArchetypeID);
			CurrentBatchChangeCount++;
			break;
		case EEntityEventType::Destroyed:
			Result.PreviousArchetype = Result.CurrentArchetype;
			Result.CurrentArchetype = 0;
			CurrentBatchChangeCount++;
			break;
		default:
			break;
		}
	}

	// When the batch starts with entity creation (ArchetypeBeforeBatch == 0)
	// but has subsequent changes, promote the Created archetype from the batch
	// into the diff base so the details view shows what changed since creation.
	// Remove it from the batch list so the substep walk doesn't compare it
	// against itself at step 0.
	if (Result.ArchetypeBeforeBatch == 0 && CurrentBatchChangeCount > 1 && CurrentBatchArchetypeIds.Num() > 0)
	{
		Result.ArchetypeBeforeBatch = CurrentBatchArchetypeIds[0];
		CurrentBatchArchetypeIds.RemoveAt(0);
		CurrentBatchChangeCount--;
	}

	Result.MergeCount = CurrentBatchChangeCount;
	Result.BatchArchetypeIds = MoveTemp(CurrentBatchArchetypeIds);
	return Result;
}

void SMassEntityDetailsView::RebuildFragmentList()
{
	using namespace UE::Mass::Trace;

	FragmentEntries.Reset();

	const uint64 ArchetypeAtScrubTime = LastDisplayedArchetypeId;
	// Use the archetype before the current frame's batch for the diff.
	// For a single change this equals the per-event previous; for merged
	// changes it captures the full delta across all steps in the frame.
	const uint64 PreviousArchetypeId = LastDisplayedArchetypeBeforeBatch;

	// Update merge info text
	if (LastDisplayedMergeCount > 1 && PreviousArchetypeId != 0)
	{
		MergeInfoText->SetText(FText::Format(
			LOCTEXT("MergedNote", "Merged {0} archetype changes in one frame"),
			FText::AsNumber(LastDisplayedMergeCount)));
		MergeInfoText->SetVisibility(EVisibility::Visible);
	}
	else
	{
		MergeInfoText->SetVisibility(EVisibility::Collapsed);
	}

	// Show/hide Previous column based on whether there is a previous archetype
	HeaderRow->SetShowGeneratedColumn(UE::MassEntityDetailsView::Private::PreviousColumnName, PreviousArchetypeId != 0);

	// Show step columns only when there are merged same-frame changes with step data
	const bool bHasSteps = LastDisplayedMergeCount > 1 && LastDisplayedArchetypeBeforeBatch != 0;
	HeaderRow->SetShowGeneratedColumn(UE::MassEntityDetailsView::Private::PreviousStepColumnName, bHasSteps);
	HeaderRow->SetShowGeneratedColumn(UE::MassEntityDetailsView::Private::CurrentStepColumnName, bHasSteps);

	if (ArchetypeAtScrubTime == 0)
	{
		if (ListView.IsValid())
		{
			ListView->RequestListRefresh();
		}
		return;
	}

	const TraceServices::IAnalysisSession* Session = IRewindDebugger::Instance()->GetAnalysisSession();
	if (!Session)
	{
		return;
	}

	const IMassTraceProvider& MassTraceProvider = ReadMassTraceProvider(*Session);

	const FArchetypeInfo* CurrentArchetype = nullptr;
	const FArchetypeInfo* PreviousArchetype = nullptr;
	{
		TraceServices::FProviderReadScopeLock ReadLock(MassTraceProvider);
		CurrentArchetype = MassTraceProvider.FindArchetypeById(ArchetypeAtScrubTime);
		if (PreviousArchetypeId != 0)
		{
			PreviousArchetype = MassTraceProvider.FindArchetypeById(PreviousArchetypeId);
		}
	}

	if (!CurrentArchetype)
	{
		if (ListView.IsValid())
		{
			ListView->RequestListRefresh();
		}
		return;
	}

	// Build fragment ID sets
	TSet<uint64> PreviousFragmentIds;
	TSet<uint64> CurrentFragmentIds;

	if (PreviousArchetype)
	{
		for (const FFragmentInfo* Fragment : PreviousArchetype->Fragments)
		{
			if (Fragment)
			{
				PreviousFragmentIds.Add(Fragment->Id);
			}
		}
	}

	for (const FFragmentInfo* Fragment : CurrentArchetype->Fragments)
	{
		if (Fragment)
		{
			CurrentFragmentIds.Add(Fragment->Id);
		}
	}

	// Step attribution for merged batches — walk through each intermediate archetype
	// in the same-frame batch to determine at which step each fragment was added/removed.
	TMap<uint64, int32> AddedAtStep;
	TMap<uint64, int32> RemovedAtStep;
	TSet<uint64> ReaddedIds;

	if (LastDisplayedMergeCount > 1 && LastDisplayedArchetypeBeforeBatch != 0)
	{
		TraceServices::FProviderReadScopeLock ReadLock(MassTraceProvider);

		// Start the substep walk from the archetype the entity was in before the
		// first event in this frame's batch — not from the previous event's archetype,
		// which may be from an entirely different point in the timeline.
		TSet<uint64> StepFragmentIds;
		if (const FArchetypeInfo* BatchStartArchetype = MassTraceProvider.FindArchetypeById(LastDisplayedArchetypeBeforeBatch))
		{
			for (const FFragmentInfo* Fragment : BatchStartArchetype->Fragments)
			{
				if (Fragment)
				{
					StepFragmentIds.Add(Fragment->Id);
				}
			}
		}

		for (int32 StepIdx = 0; StepIdx < LastBatchArchetypeIds.Num(); ++StepIdx)
		{
			const FArchetypeInfo* StepArchetype = MassTraceProvider.FindArchetypeById(LastBatchArchetypeIds[StepIdx]);
			if (!StepArchetype)
			{
				continue;
			}

			TSet<uint64> NextFragmentIds;
			for (const FFragmentInfo* Fragment : StepArchetype->Fragments)
			{
				if (Fragment)
				{
					NextFragmentIds.Add(Fragment->Id);
				}
			}

			for (uint64 Id : NextFragmentIds)
			{
				if (!StepFragmentIds.Contains(Id))
				{
					if (RemovedAtStep.Contains(Id))
					{
						ReaddedIds.Add(Id);
					}
					if (!AddedAtStep.Contains(Id))
					{
						AddedAtStep.Add(Id, StepIdx + 1);
					}
				}
			}

			for (uint64 Id : StepFragmentIds)
			{
				if (!NextFragmentIds.Contains(Id))
				{
					if (AddedAtStep.Contains(Id))
					{
						ReaddedIds.Add(Id);
					}
					if (!RemovedAtStep.Contains(Id))
					{
						RemovedAtStep.Add(Id, StepIdx + 1);
					}
				}
			}

			StepFragmentIds = MoveTemp(NextFragmentIds);
		}
	}

	// Build entries — diff between previous event's archetype and current
	for (const FFragmentInfo* Fragment : CurrentArchetype->Fragments)
	{
		if (!Fragment)
		{
			continue;
		}

		const bool bInPrevious = PreviousArchetype && PreviousFragmentIds.Contains(Fragment->Id);
		const bool bIsReadded = ReaddedIds.Contains(Fragment->Id) && bInPrevious;

		TSharedPtr<FFragmentEntry> Entry = MakeShared<FFragmentEntry>();
		Entry->Info = Fragment;
		Entry->bHasPrevious = bInPrevious;

		if (bIsReadded)
		{
			Entry->Status = EDiffStatus::Readded;
			Entry->AddedAtStep = AddedAtStep.FindRef(Fragment->Id);
			Entry->RemovedAtStep = RemovedAtStep.FindRef(Fragment->Id);
		}
		else if (!bInPrevious && PreviousArchetype)
		{
			Entry->Status = EDiffStatus::Added;
			Entry->AddedAtStep = AddedAtStep.FindRef(Fragment->Id);
		}
		else
		{
			Entry->Status = EDiffStatus::Unchanged;
		}

		FragmentEntries.Add(MoveTemp(Entry));
	}

	// Add removed fragments
	if (PreviousArchetype)
	{
		for (const FFragmentInfo* Fragment : PreviousArchetype->Fragments)
		{
			if (!Fragment || CurrentFragmentIds.Contains(Fragment->Id))
			{
				continue;
			}

			TSharedPtr<FFragmentEntry> Entry = MakeShared<FFragmentEntry>();
			Entry->Info = Fragment;
			Entry->Status = EDiffStatus::Removed;
			Entry->RemovedAtStep = RemovedAtStep.FindRef(Fragment->Id);
			Entry->bHasPrevious = true;
			FragmentEntries.Add(MoveTemp(Entry));
		}
	}

	SortFragmentEntries();

	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

EColumnSortMode::Type SMassEntityDetailsView::GetSortModeForColumn(const FName ColumnId) const
{
	return ColumnId == SortColumn ? SortMode : EColumnSortMode::None;
}

void SMassEntityDetailsView::OnSortModeChanged(const EColumnSortPriority::Type /*Priority*/, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortColumn = ColumnId;
	SortMode = InSortMode;
	SortFragmentEntries();

	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

void SMassEntityDetailsView::SortFragmentEntries()
{
	using namespace UE::MassEntityDetailsView::Private;

	const bool bAscending = SortMode != EColumnSortMode::Descending;

	FragmentEntries.Sort([this, bAscending](const TSharedPtr<FFragmentEntry>& A, const TSharedPtr<FFragmentEntry>& B)
	{
		int32 Cmp = 0;

		if (SortColumn == TypeColumnName)
		{
			Cmp = static_cast<int32>(A->Info->Type) - static_cast<int32>(B->Info->Type);
		}
		else if (SortColumn == SizeColumnName)
		{
			Cmp = static_cast<int32>(A->Info->Size) - static_cast<int32>(B->Info->Size);
		}
		else if (SortColumn == PreviousStepColumnName)
		{
			Cmp = A->RemovedAtStep - B->RemovedAtStep;
		}
		else if (SortColumn == CurrentStepColumnName)
		{
			Cmp = A->AddedAtStep - B->AddedAtStep;
		}

		// Default / tie-break: sort by fragment name
		if (Cmp == 0)
		{
			Cmp = A->Info->Name.Compare(B->Info->Name);
		}

		return bAscending ? Cmp < 0 : Cmp > 0;
	});
}

#undef LOCTEXT_NAMESPACE
