// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SHeaderRow.h"

class STextBlock;
class STableViewBase;
class ITableRow;
template <typename ItemType> class SListView;

namespace UE::Mass::Trace
{
struct FFragmentInfo;
class FEntityEventCache;
}

/**
 * Details widget for a single Mass entity.
 * Shows a diff-style table of fragments between previous and current archetype.
 */
class SMassEntityDetailsView : public SCompoundWidget
{
public:
	enum class EDiffStatus : uint8
	{
		Unchanged,
		Added,
		Removed,
		Readded
	};

	struct FFragmentEntry
	{
		const UE::Mass::Trace::FFragmentInfo* Info = nullptr;
		EDiffStatus Status = EDiffStatus::Unchanged;
		/** Step at which the fragment was added (Added/Readded). 0 = not applicable. */
		int32 AddedAtStep = 0;
		/** Step at which the fragment was removed (Removed/Readded). 0 = not applicable. */
		int32 RemovedAtStep = 0;
		bool bHasPrevious = false;
	};

	SLATE_BEGIN_ARGS(SMassEntityDetailsView) {}
		SLATE_ARGUMENT(uint64, EntityId)
		SLATE_ARGUMENT(const UE::Mass::Trace::FEntityEventCache*, Cache)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void SetScrubTime(double Time);

private:
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	struct FArchetypeResolution
	{
		uint64 CurrentArchetype = 0;
		uint64 PreviousArchetype = 0;
		/** Archetype the entity was in before the first event in the current frame's batch. */
		uint64 ArchetypeBeforeBatch = 0;
		int32 MergeCount = 0;
		TArray<uint64> BatchArchetypeIds;
	};

	void RebuildFragmentList();
	void SortFragmentEntries();
	FArchetypeResolution ResolveArchetypesAtScrubTime() const;

	EColumnSortMode::Type GetSortModeForColumn(FName ColumnId) const;
	void OnSortModeChanged(EColumnSortPriority::Type Priority, const FName& ColumnId, EColumnSortMode::Type InSortMode);

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FFragmentEntry> Entry, const TSharedRef<STableViewBase>& OwnerTable);

	uint64 EntityId = 0;
	const UE::Mass::Trace::FEntityEventCache* Cache = nullptr;
	double CurrentScrubTime = 0.0;
	uint64 LastDisplayedArchetypeId = 0;
	uint64 LastDisplayedPreviousArchetypeId = 0;
	uint64 LastDisplayedArchetypeBeforeBatch = 0;
	int32 LastDisplayedMergeCount = 0;
	TArray<uint64> LastBatchArchetypeIds;

	TArray<TSharedPtr<FFragmentEntry>> FragmentEntries;
	TSharedPtr<SListView<TSharedPtr<FFragmentEntry>>> ListView;
	TSharedPtr<SHeaderRow> HeaderRow;
	TSharedPtr<STextBlock> MergeInfoText;

	FName SortColumn;
	EColumnSortMode::Type SortMode = EColumnSortMode::Ascending;
};
