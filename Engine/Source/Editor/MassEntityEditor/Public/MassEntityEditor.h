// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/TokenizedMessage.h"
#include "MassEntityEditor.generated.h"

#define UE_API MASSENTITYEDITOR_API


USTRUCT()
struct FMassEditorNotification
{
	GENERATED_BODY()

	EMessageSeverity::Type Severity = EMessageSeverity::Error;
	FText Message;

	/** 
	 * If set to true then a clickable "see log for details" message will be added to the message log. 
	 * Clicking the message takes the user to the Output Log.
	 */
	bool bIncludeSeeOutputLogForDetails = false;

	UE_API void Show();
};

namespace UE::Mass::Editor
{
	const FName MessageLogPageName(TEXT("MassEntity"));
}

#undef UE_API
