// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"

#define UE_API MASSENTITYEDITOR_API

struct FGraphNodeClassHelper;
struct FGraphPanelNodeFactory;
class FMassEntitiesTrackCreator;
class FMassEntityTrackCreator;
class FMassRewindDebuggerRuntimeExtension;
class IMassEntityEditor;
class UWorld;
namespace UE::Mass::Trace
{
class FEntityTrackCreator;
}

/**
* The public interface to this module
*/
class FMassEntityEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override
	{
		return MenuExtensibilityManager;
	}

	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override
	{
		return ToolBarExtensibilityManager;
	}

	TSharedPtr<FGraphNodeClassHelper> GetProcessorClassCache()
	{
		return ProcessorClassCache;
	}

protected:
#if WITH_UNREAL_DEVELOPER_TOOLS
	static UE_API void OnWorldCleanup(UWorld* /*World*/, bool /*bSessionEnded*/, bool /*bCleanupResources*/);
	FDelegateHandle OnWorldCleanupHandle;
#endif // WITH_UNREAL_DEVELOPER_TOOLS

	TSharedPtr<FGraphNodeClassHelper> ProcessorClassCache;

	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	//~ RewindDebugger integration
	TUniquePtr<UE::Mass::Trace::FEntityTrackCreator> EntityTrackCreator;
	TUniquePtr<FMassRewindDebuggerRuntimeExtension> RuntimeExtension;
};

#undef UE_API
