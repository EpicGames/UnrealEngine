// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessingPhaseManager.h"
#include "Subsystems/EngineSubsystem.h"
#include "MassEntityEditorSubsystem.generated.h"

#define UE_API MASSENTITYEDITOR_API


struct FMassEntityManager;
namespace UE::Mass
{
	struct FMassEditorTickFunction;
}

/*
 * This is an Engine Subsystem that is selectively enabled using WITH_EDITOR so it is available in custom programs that run the editor with -game
 */
UCLASS(MinimalAPI)
class UMassEntityEditorSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreTick, float /*DeltaTime*/);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostTick, float /*DeltaTime*/);

	UE_API UMassEntityEditorSubsystem();
	UE_API ~UMassEntityEditorSubsystem();

	UE_API static UMassEntityEditorSubsystem* Get();

	TSharedRef<FMassEntityManager> GetMutableEntityManager() { return EntityManager; }
	TSharedRef<FMassProcessingPhaseManager> GetMutablePhaseManager() { return PhaseManager; }

	FOnPreTick& GetOnPreTickDelegate() { return OnPreTickDelegate; }
	FOnPostTick& GetOnPostTickDelegate() { return OnPostTickDelegate; }

	/** Registers a dynamic processor. This needs to be a fully formed processor and will be slotted in during the next tick. */
	UE_API void RegisterDynamicProcessor(UMassProcessor& Processor);
	/** Removes a previously registered dynamic processor of throws an assert if not found. */
	UE_API void UnregisterDynamicProcessor(UMassProcessor& Processor);

protected:
	// USubsystem interface begin
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	// USubsystem interface end

	UE_API void InitializeMassInternals();
	UE_API void StopAndCleanUp();

	friend UE::Mass::FMassEditorTickFunction;
	UE_API void Tick(float DeltaTime);

	UE::Mass::FMassEditorTickFunction* EditorTickFunction = nullptr;

	TSharedRef<FMassEntityManager> EntityManager;

	TSharedRef<FMassProcessingPhaseManager> PhaseManager;

	FMassProcessingPhaseConfig ProcessingPhasesConfig[(uint8)EMassProcessingPhase::MAX];

	/** Gets broadcast right before issuing FMassProcessingPhaseManager's phases execution */
	FOnPreTick OnPreTickDelegate;
	/** Gets broadcast right after completing FMassProcessingPhaseManager's phases execution */
	FOnPostTick OnPostTickDelegate;
	
	bool IsProcessing{ false };
	FDelegateHandle OnSettingsInitializedHandle;
};

#undef UE_API
